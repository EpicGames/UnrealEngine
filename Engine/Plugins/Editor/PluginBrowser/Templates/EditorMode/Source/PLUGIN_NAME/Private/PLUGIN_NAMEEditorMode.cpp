// Copyright Epic Games, Inc. All Rights Reserved.

#include "PLUGIN_NAMEEditorMode.h"
#include "PLUGIN_NAMEEditorModeToolkit.h"
#include "EdModeInteractiveToolsContext.h"
#include "InteractiveToolManager.h"
#include "PLUGIN_NAMEEditorModeCommands.h"
#include "Modules/ModuleManager.h"


//////////////////////////////////////////////////////////////////////////
////////////////////////////////////////////////////////////////////////// 
// AddYourTool Step 1 - include the header file for your Tools here
//////////////////////////////////////////////////////////////////////////
////////////////////////////////////////////////////////////////////////// 
#include "Tools/PLUGIN_NAMESimpleTool.h"
#include "Tools/PLUGIN_NAMEInteractiveTool.h"

// step 2: register a ToolBuilder in FPLUGIN_NAMEEditorMode::Enter() below


#define LOCTEXT_NAMESPACE "PLUGIN_NAMEEditorMode"

const FEditorModeID UPLUGIN_NAMEEditorMode::EM_PLUGIN_NAMEEditorModeId = TEXT("EM_PLUGIN_NAMEEditorMode");

FString UPLUGIN_NAMEEditorMode::SimpleToolName = TEXT("PLUGIN_NAME_ActorInfoTool");
FString UPLUGIN_NAMEEditorMode::InteractiveToolName = TEXT("PLUGIN_NAME_MeasureDistanceTool");


UPLUGIN_NAMEEditorMode::UPLUGIN_NAMEEditorMode()
{
	FModuleManager::Get().LoadModule("EditorStyle");

	// appearance and icon in the editing mode ribbon can be customized here
	Info = FEditorModeInfo(UPLUGIN_NAMEEditorMode::EM_PLUGIN_NAMEEditorModeId,
		LOCTEXT("ModeName", "PLUGIN_NAME"),
		FSlateIcon(),
		true);
}


UPLUGIN_NAMEEditorMode::~UPLUGIN_NAMEEditorMode()
{
}


void UPLUGIN_NAMEEditorMode::ActorSelectionChangeNotify()
{
}

void UPLUGIN_NAMEEditorMode::Enter()
{
	UEdMode::Enter();

	//////////////////////////////////////////////////////////////////////////
	//////////////////////////////////////////////////////////////////////////
	// AddYourTool Step 2 - register the ToolBuilders for your Tools here.
	// The string name you pass to the ToolManager is used to select/activate your ToolBuilder later.
	//////////////////////////////////////////////////////////////////////////
	////////////////////////////////////////////////////////////////////////// 
	const FPLUGIN_NAMEEditorModeCommands& SampleToolCommands = FPLUGIN_NAMEEditorModeCommands::Get();

	RegisterTool(SampleToolCommands.SimpleTool, SimpleToolName, NewObject<UPLUGIN_NAMESimpleToolBuilder>(this));
	RegisterTool(SampleToolCommands.InteractiveTool, InteractiveToolName, NewObject<UPLUGIN_NAMEInteractiveToolBuilder>(this));

	// active tool type is not relevant here, we just set to default
	GetToolManager()->SelectActiveToolType(EToolSide::Left, SimpleToolName);
}

void UPLUGIN_NAMEEditorMode::CreateToolkit()
{
	Toolkit = MakeShareable(new FPLUGIN_NAMEEditorModeToolkit);
}

TMap<FName, TArray<TSharedPtr<FUICommandInfo>>> UPLUGIN_NAMEEditorMode::GetModeCommands() const
{
	return FPLUGIN_NAMEEditorModeCommands::Get().GetCommands();
}

#undef LOCTEXT_NAMESPACE
