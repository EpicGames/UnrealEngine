// Copyright Epic Games, Inc. All Rights Reserved.

#include "PLUGIN_NAMEEditorModeCommands.h"
#include "PLUGIN_NAMEEditorMode.h"
#include "EditorStyleSet.h"

#define LOCTEXT_NAMESPACE "PLUGIN_NAMEEditorModeCommands"

FPLUGIN_NAMEEditorModeCommands::FPLUGIN_NAMEEditorModeCommands()
	: TCommands<FPLUGIN_NAMEEditorModeCommands>("PLUGIN_NAMEEditorMode",
		NSLOCTEXT("PLUGIN_NAMEEditorMode", "PLUGIN_NAMEEditorModeCommands", "PLUGIN_NAME Editor Mode"),
		NAME_None,
		FAppStyle::GetAppStyleSetName())
{
}

void FPLUGIN_NAMEEditorModeCommands::RegisterCommands()
{
	TArray <TSharedPtr<FUICommandInfo>>& ToolCommands = Commands.FindOrAdd(NAME_Default);

	UI_COMMAND(SimpleTool, "Show Actor Info", "Opens message box with info about a clicked actor", EUserInterfaceActionType::Button, FInputChord());
	ToolCommands.Add(SimpleTool);

	UI_COMMAND(InteractiveTool, "Measure Distance", "Measures distance between 2 points (click to set origin, shift-click to set end point)", EUserInterfaceActionType::ToggleButton, FInputChord());
	ToolCommands.Add(InteractiveTool);
}

TMap<FName, TArray<TSharedPtr<FUICommandInfo>>> FPLUGIN_NAMEEditorModeCommands::GetCommands()
{
	return FPLUGIN_NAMEEditorModeCommands::Get().Commands;
}

#undef LOCTEXT_NAMESPACE
