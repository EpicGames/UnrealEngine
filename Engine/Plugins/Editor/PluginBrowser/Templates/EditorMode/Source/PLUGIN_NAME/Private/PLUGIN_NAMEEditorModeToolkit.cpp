// Copyright Epic Games, Inc. All Rights Reserved.

#include "PLUGIN_NAMEEditorModeToolkit.h"
#include "PLUGIN_NAMEEditorMode.h"
#include "Engine/Selection.h"

#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "IDetailsView.h"
#include "EditorModeManager.h"

#define LOCTEXT_NAMESPACE "PLUGIN_NAMEEditorModeToolkit"

FPLUGIN_NAMEEditorModeToolkit::FPLUGIN_NAMEEditorModeToolkit()
{
}

void FPLUGIN_NAMEEditorModeToolkit::Init(const TSharedPtr<IToolkitHost>& InitToolkitHost, TWeakObjectPtr<UEdMode> InOwningMode)
{
	FModeToolkit::Init(InitToolkitHost, InOwningMode);
}

void FPLUGIN_NAMEEditorModeToolkit::GetToolPaletteNames(TArray<FName>& PaletteNames) const
{
	PaletteNames.Add(NAME_Default);
}


FName FPLUGIN_NAMEEditorModeToolkit::GetToolkitFName() const
{
	return FName("PLUGIN_NAMEEditorMode");
}

FText FPLUGIN_NAMEEditorModeToolkit::GetBaseToolkitName() const
{
	return LOCTEXT("DisplayName", "PLUGIN_NAMEEditorMode Toolkit");
}

#undef LOCTEXT_NAMESPACE
