// Copyright Epic Games, Inc. All Rights Reserved.

#include "PLUGIN_NAMEModule.h"
#include "PLUGIN_NAMEEditorModeCommands.h"

#define LOCTEXT_NAMESPACE "PLUGIN_NAMEModule"

void FPLUGIN_NAMEModule::StartupModule()
{
	// This code will execute after your module is loaded into memory; the exact timing is specified in the .uplugin file per-module

	FPLUGIN_NAMEEditorModeCommands::Register();
}

void FPLUGIN_NAMEModule::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.

	FPLUGIN_NAMEEditorModeCommands::Unregister();
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FPLUGIN_NAMEModule, PLUGIN_NAME)