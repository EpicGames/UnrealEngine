// Copyright Epic Games, Inc. All Rights Reserved.

#include "PLUGIN_NAMESimpleTool.h"
#include "InteractiveToolManager.h"
#include "ToolBuilderUtil.h"
#include "CollisionQueryParams.h"
#include "Engine/World.h"
#include "Engine/HitResult.h"
#include "Misc/MessageDialog.h"

// localization namespace
#define LOCTEXT_NAMESPACE "PLUGIN_NAMESimpleTool"

/*
 * ToolBuilder implementation
 */

UInteractiveTool* UPLUGIN_NAMESimpleToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	UPLUGIN_NAMESimpleTool* NewTool = NewObject<UPLUGIN_NAMESimpleTool>(SceneState.ToolManager);
	NewTool->SetWorld(SceneState.World);
	return NewTool;
}



/*
 * ToolProperties implementation
 */

UPLUGIN_NAMESimpleToolProperties::UPLUGIN_NAMESimpleToolProperties()
{
	ShowExtendedInfo = true;
}


/*
 * Tool implementation
 */

UPLUGIN_NAMESimpleTool::UPLUGIN_NAMESimpleTool()
{
}


void UPLUGIN_NAMESimpleTool::SetWorld(UWorld* World)
{
	this->TargetWorld = World;
}


void UPLUGIN_NAMESimpleTool::Setup()
{
	USingleClickTool::Setup();

	Properties = NewObject<UPLUGIN_NAMESimpleToolProperties>(this);
	AddToolPropertySource(Properties);
}


void UPLUGIN_NAMESimpleTool::OnClicked(const FInputDeviceRay& ClickPos)
{
	// we will create actor at this position
	FVector NewActorPos = FVector::ZeroVector;

	// cast ray into world to find hit position
	FVector RayStart = ClickPos.WorldRay.Origin;
	FVector RayEnd = ClickPos.WorldRay.PointAt(99999999.f);
	FCollisionObjectQueryParams QueryParams(FCollisionObjectQueryParams::AllObjects);
	FHitResult Result;
	if (TargetWorld->LineTraceSingleByObjectType(Result, RayStart, RayEnd, QueryParams))
	{
		if (AActor* ClickedActor = Result.GetActor())
		{
			FText ActorInfoMsg;

			if (Properties->ShowExtendedInfo)
			{
				ActorInfoMsg = FText::Format(LOCTEXT("ExtendedActorInfo", "Name: {0}\nClass: {1}"), 
					FText::FromString(ClickedActor->GetName()), 
					FText::FromString(ClickedActor->GetClass()->GetName())
				);
			}
			else
			{
				ActorInfoMsg = FText::Format(LOCTEXT("BasicActorInfo", "Name: {0}"), FText::FromString(Result.GetActor()->GetName()));
			}

			FText Title = LOCTEXT("ActorInfoDialogTitle", "Actor Info");
			// JAH TODO: consider if we can highlight the actor prior to opening the dialog box or make it non-modal
			FMessageDialog::Open(EAppMsgType::Ok, ActorInfoMsg, Title);
		}
	}
}


#undef LOCTEXT_NAMESPACE