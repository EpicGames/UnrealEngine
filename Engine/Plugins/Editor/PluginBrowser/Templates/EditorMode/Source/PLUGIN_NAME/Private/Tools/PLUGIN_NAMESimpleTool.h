// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NoExportTypes.h"
#include "InteractiveToolBuilder.h"
#include "BaseTools/SingleClickTool.h"
#include "PLUGIN_NAMESimpleTool.generated.h"

/**
 * Builder for UPLUGIN_NAMESimpleTool
 */
UCLASS()
class PLUGIN_NAME_API UPLUGIN_NAMESimpleToolBuilder : public UInteractiveToolBuilder
{
	GENERATED_BODY()

public:
	virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override { return true; }
	virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;
};



/**
 * Settings UObject for UPLUGIN_NAMESimpleTool. This UClass inherits from UInteractiveToolPropertySet,
 * which provides an OnModified delegate that the Tool will listen to for changes in property values.
 */
UCLASS(Transient)
class PLUGIN_NAME_API UPLUGIN_NAMESimpleToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	UPLUGIN_NAMESimpleToolProperties();

	/** If enabled, dialog should display extended information about the actor clicked on. Otherwise, only basic info will be shown. */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DisplayName = "Show Extended Info"))
	bool ShowExtendedInfo;
};




/**
 * UPLUGIN_NAMESimpleTool is an example Tool that opens a message box displaying info about an actor that the user
 * clicks left mouse button. All the action is in the ::OnClicked handler.
 */
UCLASS()
class PLUGIN_NAME_API UPLUGIN_NAMESimpleTool : public USingleClickTool
{
	GENERATED_BODY()

public:
	UPLUGIN_NAMESimpleTool();

	virtual void SetWorld(UWorld* World);

	virtual void Setup() override;

	virtual void OnClicked(const FInputDeviceRay& ClickPos);


protected:
	UPROPERTY()
	TObjectPtr<UPLUGIN_NAMESimpleToolProperties> Properties;


protected:
	/** target World we will raycast into to find actors */
	UWorld* TargetWorld;
};