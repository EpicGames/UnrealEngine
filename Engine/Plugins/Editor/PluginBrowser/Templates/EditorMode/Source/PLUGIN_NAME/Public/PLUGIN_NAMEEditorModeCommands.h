// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"

/**
 * This class contains info about the full set of commands used in this editor mode.
 */
class FPLUGIN_NAMEEditorModeCommands : public TCommands<FPLUGIN_NAMEEditorModeCommands>
{
public:
	FPLUGIN_NAMEEditorModeCommands();

	virtual void RegisterCommands() override;
	static TMap<FName, TArray<TSharedPtr<FUICommandInfo>>> GetCommands();

	TSharedPtr<FUICommandInfo> SimpleTool;
	TSharedPtr<FUICommandInfo> InteractiveTool;

protected:
	TMap<FName, TArray<TSharedPtr<FUICommandInfo>>> Commands;
};
