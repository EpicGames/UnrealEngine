// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"

/**
 * This is the module definition for the editor mode. You can implement custom functionality
 * as your plugin module starts up and shuts down. See IModuleInterface for more extensibility options.
 */
class FPLUGIN_NAMEModule : public IModuleInterface
{
public:

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};
